{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Significance Manager",
	"Description": "The significance manager plugin provides an extensible framework for allowing games to calculate the significance of an object and change behavior in response.",
	"Category": "Performance",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SignificanceManager",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"IsExperimentalVersion": false
}