// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignificanceManager.h"
#include "Engine/Canvas.h"
#include "GameFramework/HUD.h"
#include "Engine/Engine.h"
#include "Modules/ModuleManager.h"
#include "Async/ParallelFor.h"
#include "Stats/StatsTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SignificanceManager)

#if ALLOW_CONSOLE
#include "Engine/Console.h"
#include "ConsoleSettings.h"
#endif // ALLOW_CONSOLE

/* Module definition for significance manager. Owns the references to created significance managers*/
class SIGNIFICANCEMANAGER_API FSignificanceManagerModule : public FDefaultModuleImpl, public FGCObject
{
public:
	// Begin IModuleInterface overrides
	virtual void StartupModule() override;
	// End IModuleInterface overrides

	// Begin FGCObject overrides
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSignificanceManagerModule");
	}
	// End FGCObject overrides

	// Returns the significance manager for the specified World
	FORCEINLINE static USignificanceManager* Get(const UWorld* World)
	{
		return WorldSignificanceManagers.FindRef(World);
	}

private:

	// Callback function registered with global world delegates to instantiate significance manager when a game world is created
	static void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);

	// Callback function registered with global world delegates to cleanup significance manager when a game world is destroyed
	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	// Callback function registered with HUD to supply debug info when ShowDebug SignificanceManager has been entered on the console
	static void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

#if ALLOW_CONSOLE
	// Callback function registered with Console to inject show debug auto complete command
	static void PopulateAutoCompleteEntries(TArray<FAutoCompleteCommand>& AutoCompleteList);
#endif // ALLOW_CONSOLE

	// Map of worlds to their significance manager
	static TMap<const UWorld*, TObjectPtr<USignificanceManager>> WorldSignificanceManagers;

	// Cached class for instantiating significance manager
	static TSubclassOf<USignificanceManager>  SignificanceManagerClass;
};


IMPLEMENT_MODULE( FSignificanceManagerModule, SignificanceManager );

DECLARE_CYCLE_STAT(TEXT("Update Total"), STAT_SignificanceManager_Update, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Significance Update"), STAT_SignificanceManager_SignificanceUpdate, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Significance Sort"), STAT_SignificanceManager_SignificanceSort, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Unregister Object"), STAT_SignificanceManager_UnregisterObject, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Significance Check"), STAT_SignificanceManager_SignificanceCheck, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Register Object"), STAT_SignificanceManager_RegisterObject, STATGROUP_SignificanceManager);
DECLARE_CYCLE_STAT(TEXT("Initial Significance Update"), STAT_SignificanceManager_InitialSignificanceUpdate, STATGROUP_SignificanceManager);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Managed Objects"), STAT_SignificanceManager_NumObjects, STATGROUP_SignificanceManager);

DEFINE_LOG_CATEGORY(LogSignificance);

bool CompareBySignificanceAscending(const USignificanceManager::FManagedObjectInfo& A, const USignificanceManager::FManagedObjectInfo& B) 
{ 
	return A.GetSignificance() < B.GetSignificance(); 
}

bool CompareBySignificanceDescending(const USignificanceManager::FManagedObjectInfo& A, const USignificanceManager::FManagedObjectInfo& B) 
{ 
	return A.GetSignificance() > B.GetSignificance(); 
}

typedef bool CompareFunctionType(const USignificanceManager::FManagedObjectInfo&,const USignificanceManager::FManagedObjectInfo&);

CompareFunctionType& PickCompareBySignificance(const bool bAscending)
{
	return (bAscending ? CompareBySignificanceAscending : CompareBySignificanceDescending);
}

TMap<const UWorld*, TObjectPtr<USignificanceManager>> FSignificanceManagerModule::WorldSignificanceManagers;
TSubclassOf<USignificanceManager> FSignificanceManagerModule::SignificanceManagerClass;

void FSignificanceManagerModule::StartupModule()
{
	FWorldDelegates::OnPreWorldInitialization.AddStatic(&FSignificanceManagerModule::OnWorldInit);
	FWorldDelegates::OnPostWorldCleanup.AddStatic(&FSignificanceManagerModule::OnWorldCleanup);
	if (!IsRunningDedicatedServer())
	{
		AHUD::OnShowDebugInfo.AddStatic(&FSignificanceManagerModule::OnShowDebugInfo);
	}

#if ALLOW_CONSOLE
	UConsole::RegisterConsoleAutoCompleteEntries.AddStatic(&FSignificanceManagerModule::PopulateAutoCompleteEntries);
#endif // ALLOW_CONSOLE
}

void FSignificanceManagerModule::AddReferencedObjects( FReferenceCollector& Collector )
{
	for (auto& WorldSignificanceManagerPair : WorldSignificanceManagers)
	{
		Collector.AddReferencedObject(WorldSignificanceManagerPair.Value, WorldSignificanceManagerPair.Key);
	}
	UClass* SignificanceManagerClassPtr = *SignificanceManagerClass;
	Collector.AddReferencedObject(SignificanceManagerClass.GetGCPtr());
	SignificanceManagerClass = SignificanceManagerClassPtr; // Since pointer can be modified by AddReferencedObject
}

void FSignificanceManagerModule::OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS)
{
	if (World->IsGameWorld())
	{
		if (*SignificanceManagerClass == nullptr)
		{
			SignificanceManagerClass = LoadClass<USignificanceManager>(nullptr, *GetDefault<USignificanceManager>()->SignificanceManagerClassName.ToString());
		}

		if (*SignificanceManagerClass != nullptr)
		{
			const USignificanceManager* ManagerToCreateDefault = SignificanceManagerClass->GetDefaultObject<USignificanceManager>();
			if ((ManagerToCreateDefault->bCreateOnServer && !IsRunningClientOnly()) || (ManagerToCreateDefault->bCreateOnClient && !IsRunningDedicatedServer()))
			{
				WorldSignificanceManagers.Add(World, NewObject<USignificanceManager>(World, SignificanceManagerClass));
			}
		}
	}
}

void FSignificanceManagerModule::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	WorldSignificanceManagers.Remove(World);
}

void FSignificanceManagerModule::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_SignificanceManager("SignificanceManager");
	if (Canvas && HUD->ShouldDisplayDebug(NAME_SignificanceManager))
	{
		if (USignificanceManager* SignificanceManager = Get(HUD->GetWorld()))
		{
			SignificanceManager->OnShowDebugInfo(HUD, Canvas, DisplayInfo, YL, YPos);
		}
	}
}

#if ALLOW_CONSOLE
void FSignificanceManagerModule::PopulateAutoCompleteEntries(TArray<FAutoCompleteCommand>& AutoCompleteList)
{
	const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

	AutoCompleteList.AddDefaulted();
	
	FAutoCompleteCommand& AutoCompleteCommand = AutoCompleteList.Last();
	AutoCompleteCommand.Command = TEXT("showdebug SIGNIFICANCEMANAGER");
	AutoCompleteCommand.Desc = TEXT("Toggles display of significance manager calculations");
	AutoCompleteCommand.Color = ConsoleSettings->AutoCompleteCommandColor;
}
#endif // ALLOW_CONSOLE

USignificanceManager::USignificanceManager()
	: Super()
{
	SignificanceManagerClassName = FSoftClassPath(GetClass()); 

	bCreateOnClient = true;
	bCreateOnServer = true;
	bSortSignificanceAscending = false;
}

void USignificanceManager::BeginDestroy()
{
	Super::BeginDestroy();

	for (const auto& ObjectToObjectInfoPair : ManagedObjects)
	{
		delete ObjectToObjectInfoPair.Value;
	}
	ManagedObjects.Reset();
	ManagedObjectsByTag.Reset();
	ObjArray.Reset();
	ObjWithSequentialPostWork.Reset();
}

UWorld* USignificanceManager::GetWorld()const
{
	return CastChecked<UWorld>(GetOuter());
}

void USignificanceManager::RegisterObject(UObject* Object, const FName Tag, FManagedObjectSignificanceFunction SignificanceFunction, USignificanceManager::EPostSignificanceType PostSignificanceType, FManagedObjectPostSignificanceFunction PostSignificanceFunction)
{
	FManagedObjectInfo* ObjectInfo = new FManagedObjectInfo(Object, Tag, SignificanceFunction, PostSignificanceType, PostSignificanceFunction);
	RegisterManagedObject(ObjectInfo);
}

void USignificanceManager::RegisterManagedObject(FManagedObjectInfo* ObjectInfo)
{
	INC_DWORD_STAT(STAT_SignificanceManager_NumObjects);
	SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_RegisterObject);

	UObject* Object = ObjectInfo->GetObject();

	check(Object);
	if (!ensureMsgf(!ManagedObjects.Contains(Object), TEXT("'%s' already added to significance manager. Original Tag: '%s' New Tag: '%s'"), *Object->GetName(), *ManagedObjects.FindChecked(Object)->GetTag().ToString(), *ObjectInfo->GetTag().ToString()))
	{
		UE_LOGF(LogSignificance, Warning, "'%ls' already added to significance manager. Original Tag: '%ls' New Tag: '%ls'", *Object->GetName(), *ManagedObjects.FindChecked(Object)->GetTag().ToString(), *ObjectInfo->GetTag().ToString());
		delete ObjectInfo;
		return;
	}

	if (ObjectInfo->GetPostSignificanceType() == EPostSignificanceType::Sequential)
	{
		ObjWithSequentialPostWork.Add({ ObjectInfo, ObjectInfo->GetSignificance() });
	}

	// Calculate initial significance
	if (Viewpoints.Num())
	{
		SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_InitialSignificanceUpdate);
		ObjectInfo->UpdateSignificance(Viewpoints,bSortSignificanceAscending);

		if (ObjectInfo->GetPostSignificanceType() == EPostSignificanceType::Sequential)
		{
			ObjectInfo->PostSignificanceFunction(ObjectInfo, 1.f, ObjectInfo->GetSignificance(), false);
		}
	}

	ManagedObjects.Add(Object, ObjectInfo);
	ObjArray.Add(ObjectInfo);
	TArray<FManagedObjectInfo*>& ManagedObjectInfos = ManagedObjectsByTag.FindOrAdd(ObjectInfo->GetTag());

	if (ManagedObjectInfos.Num() > 0)
	{
		// Insert in to the sorted list
		int32 LowIndex = 0;
		int32 HighIndex = ManagedObjectInfos.Num() - 1;
		auto CompareFunction = PickCompareBySignificance(bSortSignificanceAscending);
		while (true)
		{
			int32 MidIndex = LowIndex + (HighIndex - LowIndex) / 2;
			if (CompareFunction(*ObjectInfo, *ManagedObjectInfos[MidIndex]))
			{
				if (LowIndex == MidIndex)
				{
					ManagedObjectInfos.Insert(ObjectInfo, LowIndex);
					break;
				}
				else
				{
					HighIndex = MidIndex - 1;
				}
			}
			else if (LowIndex == HighIndex)
			{
				ManagedObjectInfos.Insert(ObjectInfo, LowIndex + 1);
				break;
			}
			else
			{
				LowIndex = MidIndex + 1;
			}
		}
	}
	else
	{
		ManagedObjectInfos.Add(ObjectInfo);
	}
}

void USignificanceManager::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	USignificanceManager* This = CastChecked<USignificanceManager>(InThis);
	// Do not allow eliminating references here so that we don't have to deal with cleaning up management info during GC.
	// All managed objects should be removed from the significance manager before being marked for explicit destruction.
	Collector.AllowEliminatingReferences(false);
	// This explicit template instantiation prevents the use of the overload which expects the map's value type to be a UObject*
	Collector.AddReferencedObjects<UObject, FManagedObjectInfo*>(This->ManagedObjects, This);
	Collector.AllowEliminatingReferences(true);
}

void USignificanceManager::UnregisterObject(UObject* Object)
{
	DEC_DWORD_STAT(STAT_SignificanceManager_NumObjects);
	SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_UnregisterObject);

	FManagedObjectInfo* ObjectInfo;
	if (ManagedObjects.RemoveAndCopyValue(Object, ObjectInfo))
	{
		if (ObjectInfo->GetPostSignificanceType() == EPostSignificanceType::Sequential)
		{
			const int32 Index = ObjWithSequentialPostWork.IndexOfByPredicate([ObjectInfo](const FSequentialPostWorkPair& WorkPair) { return WorkPair.ObjectInfo == ObjectInfo; });
			if (Index != -1)
			{
				ObjWithSequentialPostWork.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}

		ObjArray.RemoveSingleSwap(ObjectInfo, EAllowShrinking::No);

		TArray<FManagedObjectInfo*>& ObjectsWithTag = ManagedObjectsByTag.FindChecked(ObjectInfo->GetTag());
		if (ObjectsWithTag.Num() == 1)
		{
			check(ObjectsWithTag[0] == ObjectInfo);
			ManagedObjectsByTag.Remove(ObjectInfo->GetTag());
		}
		else
		{
			ObjectsWithTag.RemoveSingle(ObjectInfo);
		}

		if (ObjectInfo->PostSignificanceFunction)
		{
			ObjectInfo->PostSignificanceFunction(ObjectInfo, ObjectInfo->Significance, 1.0f, true);
		}

		delete ObjectInfo;
	}
}

void USignificanceManager::UnregisterAll(FName Tag)
{
	if (TArray<FManagedObjectInfo*>* ObjectsWithTag = ManagedObjectsByTag.Find(Tag))
	{
		for (FManagedObjectInfo* ManagedObj : *ObjectsWithTag)
		{
			if (ManagedObj->GetPostSignificanceType() == EPostSignificanceType::Sequential)
			{
				const int32 Index = ObjWithSequentialPostWork.IndexOfByPredicate([ManagedObj](const FSequentialPostWorkPair& WorkPair) { return WorkPair.ObjectInfo == ManagedObj; });
				if (Index != -1)
				{
					ObjWithSequentialPostWork.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}

			ObjArray.RemoveSingleSwap(ManagedObj, EAllowShrinking::No);
			ManagedObjects.Remove(ManagedObj->GetObject());
			if (ManagedObj->PostSignificanceFunction != nullptr)
			{
				ManagedObj->PostSignificanceFunction(ManagedObj, ManagedObj->Significance, 1.0f, true);
			}

			delete ManagedObj;
		}
		ManagedObjectsByTag.Remove(Tag);
	}
}

const TArray<USignificanceManager::FManagedObjectInfo*>& USignificanceManager::GetManagedObjects(const FName Tag) const
{
	if (const TArray<FManagedObjectInfo*>* ObjectsWithTag = ManagedObjectsByTag.Find(Tag))
	{
		return *ObjectsWithTag;
	}

	static const TArray<FManagedObjectInfo*> EmptySet;
	return EmptySet;
}

USignificanceManager::FManagedObjectInfo* USignificanceManager::GetManagedObject(UObject* Object) const
{
	if (const auto* Info = ManagedObjects.Find(Object))
	{
		return (*Info);
	}

	return nullptr;
}

const USignificanceManager::USignificanceManager::FManagedObjectInfo* USignificanceManager::GetManagedObject(const UObject* Object) const
{
	if (const auto* Info = ManagedObjects.Find(Object))
	{
		return (*Info);
	}

	return nullptr;
}

void USignificanceManager::GetManagedObjects(TArray<USignificanceManager::FManagedObjectInfo*>& OutManagedObjects, bool bInSignificanceOrder) const
{
	OutManagedObjects.Reserve(ManagedObjects.Num());
	for (const TPair<FName, TArray<FManagedObjectInfo*>>& TagToObjectInfoArrayPair : ManagedObjectsByTag)
	{
		OutManagedObjects.Append(TagToObjectInfoArrayPair.Value);
	}
	if (bInSignificanceOrder)
	{
		OutManagedObjects.Sort(PickCompareBySignificance(bSortSignificanceAscending));
	}
}

float USignificanceManager::GetSignificance(const UObject* Object) const
{
	SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_SignificanceCheck);
	float Significance = 0.f;
	if (const auto* Info = ManagedObjects.Find(Object))
	{
		Significance = (*Info)->GetSignificance();
	}
	return Significance;
}

bool USignificanceManager::QuerySignificance(const UObject* Object, float& OutSignificance) const
{
	SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_SignificanceCheck);
	if (const auto* Info = ManagedObjects.Find(Object))
	{
		OutSignificance = (*Info)->GetSignificance();
		return true;
	}
	else
	{
		OutSignificance = 0.0f;
		return false;
	}
}

USignificanceManager* USignificanceManager::Get(const UWorld* World)
{
	return FSignificanceManagerModule::Get(World);
}

void USignificanceManager::FManagedObjectInfo::UpdateSignificance(const TArray<FTransform>& InViewpoints, const bool bSortAscending)
{
	float OldSignificance = Significance;
	if (InViewpoints.Num())
	{
		if (bSortAscending)
		{
			Significance = TNumericLimits<float>::Max();
			for (const FTransform& Viewpoint : InViewpoints)
			{
				const float ViewpointSignificance = SignificanceFunction(this, Viewpoint);
				if (ViewpointSignificance < Significance)
				{
					Significance = ViewpointSignificance;
				}
			}
		}
		else
		{
			Significance = TNumericLimits<float>::Lowest();
			for (const FTransform& Viewpoint : InViewpoints)
			{
				const float ViewpointSignificance = SignificanceFunction(this, Viewpoint);
				if (ViewpointSignificance > Significance)
				{
					Significance = ViewpointSignificance;
				}
			}
		}
	}
	else
	{
		Significance = 0.f;
	}

	if (PostSignificanceType == EPostSignificanceType::Concurrent)
	{
		PostSignificanceFunction(this, OldSignificance, Significance, false);
	}
}

void USignificanceManager::Update(TArrayView<const FTransform> InViewpoints)
{
	Viewpoints.Reset(InViewpoints.Num());
	Viewpoints.Append(InViewpoints.GetData(), InViewpoints.Num());

	SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_Update);

	{
		SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_SignificanceUpdate);

		// Update old significance values
		for (FSequentialPostWorkPair& SequentialPostWorkPair : ObjWithSequentialPostWork)
		{
			SequentialPostWorkPair.OldSignificance = SequentialPostWorkPair.ObjectInfo->GetSignificance();
		}

		// Copy the data we'll be working on this frame to avoid issues if objects are registered/unregistered while it runs.
		// We don't need to clear them afterwards as we always overwrite them fully.
		ObjArrayCopy = ObjArray;
		ObjWithSequentialPostWorkCopy = ObjWithSequentialPostWork;

		ParallelFor(ObjArrayCopy.Num(),
			[&](int32 Index)
		{
			FManagedObjectInfo* ObjectInfo = ObjArrayCopy[Index];

			checkSlow(ObjectInfo->GetObject()->IsValidLowLevel());

			ObjectInfo->UpdateSignificance(Viewpoints,bSortSignificanceAscending);
		});

		for (const FSequentialPostWorkPair& SequentialPostWorkPair : ObjWithSequentialPostWorkCopy)
		{
			FManagedObjectInfo* ObjectInfo = SequentialPostWorkPair.ObjectInfo; 
			ObjectInfo->PostSignificanceFunction(ObjectInfo, SequentialPostWorkPair.OldSignificance, ObjectInfo->GetSignificance(), false);
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_SignificanceManager_SignificanceSort);
		for (TPair<FName, TArray<FManagedObjectInfo*>>& TagToObjectInfoArrayPair : ManagedObjectsByTag)
		{
			TagToObjectInfoArrayPair.Value.StableSort(PickCompareBySignificance(bSortSignificanceAscending));
		}
	}
}

static int32 GSignificanceManagerObjectsToShow = 15;
static FAutoConsoleVariableRef CVarSignificanceManagerObjectsToShow(
	TEXT("SigMan.ObjectsToShow"),
	GSignificanceManagerObjectsToShow,
	TEXT("How many objects to display when ShowDebug SignificanceManager is enabled.\n"),
	ECVF_Cheat
	);

static FAutoConsoleVariable CVarSignificanceManagerFilterTag(
	TEXT("SigMan.FilterTag"),
	TEXT(""),
	TEXT("Only display objects with the specified filter tag.  If None objects with any will be displayed.\n"),
	ECVF_Cheat
	);


void USignificanceManager::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_SignificanceManager("SignificanceManager");
	if (Canvas && HUD->ShouldDisplayDebug(NAME_SignificanceManager))
	{
		if (HasAnyFlags(RF_ClassDefaultObject))
		{
			if (USignificanceManager* SignificanceManager = Get(HUD->GetWorld()))
			{
				SignificanceManager->OnShowDebugInfo(HUD, Canvas, DisplayInfo, YL, YPos);
			}
		}
		else
		{
			FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
			DisplayDebugManager.SetFont(GEngine->GetSmallFont());
			DisplayDebugManager.SetDrawColor(FColor::Red);
			DisplayDebugManager.DrawString(FString::Printf(TEXT("SIGNIFICANCE MANAGER - %d Managed Objects"), ManagedObjects.Num()));

			const FName SignificanceManagerTag(*CVarSignificanceManagerFilterTag->GetString());
			TArray<FManagedObjectInfo*> AllObjects;
			const TArray<FManagedObjectInfo*>& ObjectsToShow = (SignificanceManagerTag.IsNone() ? AllObjects : GetManagedObjects(SignificanceManagerTag));
			if (SignificanceManagerTag.IsNone())
			{
				GetManagedObjects(AllObjects, true);
			}

			DisplayDebugManager.SetDrawColor(FColor::White);
			const int32 NumObjectsToShow = FMath::Min(GSignificanceManagerObjectsToShow, ObjectsToShow.Num());
			for (int32 Index = 0; Index < NumObjectsToShow; ++Index)
			{
				const FManagedObjectInfo* ObjectInfo = ObjectsToShow[Index];
				const FString Str = FString::Printf(TEXT("%6.3f - %s (%s)"), ObjectInfo->GetSignificance(), *ObjectInfo->GetObject()->GetName(), *ObjectInfo->GetTag().ToString());
				DisplayDebugManager.DrawString(Str);
			}
		}
	}
}
