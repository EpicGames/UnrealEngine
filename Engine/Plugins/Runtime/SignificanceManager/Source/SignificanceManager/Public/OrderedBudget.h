// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

// Manages how to allocate a budget of levels to an ordered list of items (e.g., how to distribute minimum LOD levels to the N closest characters)
struct FOrderedBudget
{
public:
	FOrderedBudget()
		: ValueForOutOfBounds(0)
	{
	}

	// Returns the budget number for the Index-th closest character
	int32 GetBudgetForIndex(int32 Index) const
	{
		Index = Index / BudgetValuesScale;
		return (Index < BudgetValues.Num()) ? BudgetValues[Index] : ValueForOutOfBounds;
	}

	// Returns the number of items specified in the budget
	int32 GetBudgetLength() const
	{
		return BudgetValues.Num();
	}

	// Recreates the budget given a specification string. For example, the specification "0,2,3,5" would correspond to the following budget:
	//   Level0: 0
	//   Level1: 2
	//   Level2: 3
	//   Level3: 5
	// Creating a table that contains:
	//   1,1,2,2,2,3,3,3,3,3 (out of bounds = 4)
	// Returns true if the budget was modified, or false if the existing budget already matched
	bool RecreateBudget(const FString& Specification)
	{
		const bool bSpecificationDiffers = Specification != BudgetString;

		if (bSpecificationDiffers)
		{
			BudgetValues.Reset();
			BudgetString = Specification;

			TArray<FString> BudgetStrings;
			BudgetString.ParseIntoArray(/*out*/ BudgetStrings, TEXT(","), /*bCullEmpty=*/ false);

			int32 LevelIndex = 0;
			for (const FString& BudgetItem : BudgetStrings)
			{
				const int32 CountForThisLevel = FCString::Atoi(*BudgetItem);
				check(CountForThisLevel >= 0);

				for (int32 ThisLevelIndex = 0; ThisLevelIndex < CountForThisLevel; ++ThisLevelIndex)
				{
					BudgetValues.Add(LevelIndex);
				}

				++LevelIndex;
			}

			ValueForOutOfBounds = LevelIndex;
		}

		return bSpecificationDiffers;
	}

	void SetBudgetScale(float InScale)
	{
		if (InScale > 0)
		{
			BudgetValuesScale = InScale;
		}
	}

private:
	// This is the budget value for the i-th closest character (e.g., there will be be 10 entries if the budgets ranges sum to cover 10)
	TArray<int32> BudgetValues;

	// The scalar to apply to the budget values, it does so by compressing the index used when requesting the budget
	float BudgetValuesScale = 1.0f;

	// This is the budget value for things with an index further away than BudgetByIndex.Num()
	int32 ValueForOutOfBounds;

	// My budget string
	FString BudgetString;
};
