// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "SignificanceManager.generated.h"

#define UE_API SIGNIFICANCEMANAGER_API

class AHUD;
class FDebugDisplayInfo;
class UCanvas;
class USignificanceManager;
struct FAutoCompleteCommand;

DECLARE_STATS_GROUP(TEXT("Significance Manager"), STATGROUP_SignificanceManager, STATCAT_Advanced);
DECLARE_LOG_CATEGORY_EXTERN(LogSignificance, Log, All);

/* The significance manager provides a framework for registering objects by tag to each have a significance
 * value calculated from which a game specific subclass and game logic can make decisions about what level
 * of detail objects should be at, tick frequency, whether to spawn effects, and other such functionality
 *
 * Each object that is registered must have a corresponding unregister event or else a dangling Object reference will
 * be left resulting in an eventual crash once the Object has been garbage collected.
 *
 * Each user of the significance manager is expected to call the Update function from the appropriate location in the
 * game code.  GameViewportClient::Tick may often serve as a good place to do this.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class USignificanceManager : public UObject
{
	GENERATED_BODY()

public:
	typedef TFunction<float(UObject*, const FTransform&)> FSignificanceFunction;
	typedef TFunction<void(UObject*, float, float, bool)> FPostSignificanceFunction;

	struct FManagedObjectInfo;

	typedef TFunction<float(FManagedObjectInfo*, const FTransform&)> FManagedObjectSignificanceFunction;
	typedef TFunction<void(FManagedObjectInfo*, float, float, bool)> FManagedObjectPostSignificanceFunction;


	enum class EPostSignificanceType : uint8
	{
		// The object has no post work to be done
		None,
		// The object's post work can be done safely in parallel
		Concurrent,
		// The object's post work must be done sequentially
		Sequential
	};

	// Hacky base class to avoid 8 bytes of padding after the vtable
	struct FManagedObjectInfoFixLayout
	{
		virtual ~FManagedObjectInfoFixLayout() = default;
	};

	struct FManagedObjectInfo : public FManagedObjectInfoFixLayout
	{
		FManagedObjectInfo()
			: Object(nullptr)
			, Significance(-1.0f)
			, PostSignificanceType(EPostSignificanceType::None)
		{
		}

		FManagedObjectInfo(UObject* InObject, FName InTag, FManagedObjectSignificanceFunction InSignificanceFunction, EPostSignificanceType InPostSignificanceType = EPostSignificanceType::None, FManagedObjectPostSignificanceFunction InPostSignificanceFunction = nullptr)
			: Object(InObject)
			, Tag(InTag)
			, Significance(1.0f)
			, PostSignificanceType(InPostSignificanceType)
			, SignificanceFunction(MoveTemp(InSignificanceFunction))
			, PostSignificanceFunction(MoveTemp(InPostSignificanceFunction))
		{
			if (PostSignificanceFunction)
			{
				ensure(PostSignificanceType != EPostSignificanceType::None);
			}
			else
			{
				ensure(PostSignificanceType == EPostSignificanceType::None);
				PostSignificanceType = EPostSignificanceType::None;
			}
		}

		UObject* GetObject() const { return Object; }
		FName GetTag() const { return Tag; }
		float GetSignificance() const { return Significance; }
		FManagedObjectSignificanceFunction GetSignificanceFunction() const { return SignificanceFunction; }
		EPostSignificanceType GetPostSignificanceType() const { return PostSignificanceType; }
		FManagedObjectPostSignificanceFunction GetPostSignificanceNotifyDelegate() const { return PostSignificanceFunction; }

	private:
		UObject* Object;
		FName Tag;
		float Significance;
		EPostSignificanceType PostSignificanceType;

		FManagedObjectSignificanceFunction SignificanceFunction;
		FManagedObjectPostSignificanceFunction PostSignificanceFunction;

		UE_API void UpdateSignificance(const TArray<FTransform>& ViewPoints, const bool bSortSignificanceAscending);

		// Allow SignificanceManager to call UpdateSignificance
		friend USignificanceManager;
	};

	UE_API USignificanceManager();

	// Begin UObject overrides
	UE_API virtual void BeginDestroy() override;
	UE_API virtual UWorld* GetWorld() const override final;
	// End UObject overrides

	// Overridable function to update the managed objects' significance
	UE_API virtual void Update(TArrayView<const FTransform> Viewpoints);

	// Overridable function used to register an object as managed by the significance manager
	UE_API virtual void RegisterObject(UObject* Object, FName Tag, FManagedObjectSignificanceFunction SignificanceFunction, EPostSignificanceType InPostSignificanceType = EPostSignificanceType::None, FManagedObjectPostSignificanceFunction InPostSignificanceFunction = nullptr);

	// Overridable function used to unregister an object as managed by the significance manager
	UE_API virtual void UnregisterObject(UObject* Object);

	// Unregisters all objects with the specified tag.
	UE_API void UnregisterAll(FName Tag);

	// Returns objects of specified tag, Tag must be specified or else an empty array will be returned
	UE_API const TArray<FManagedObjectInfo*>& GetManagedObjects(FName Tag) const;

	// Returns all managed objects regardless of tag
	UE_API void GetManagedObjects(TArray<FManagedObjectInfo*>& OutManagedObjects, bool bInSignificanceOrder = false) const;

	// Returns the managed object for the passed-in object, if any. Otherwise returns nullptr
	UE_API USignificanceManager::FManagedObjectInfo* GetManagedObject(UObject* Object) const;

	// Returns the managed object for the passed-in object, if any. Otherwise returns nullptr
	UE_API const USignificanceManager::FManagedObjectInfo* GetManagedObject(const UObject* Object) const;

	// Returns the significance value for a given object, returns 0 if object is not managed
	UE_API float GetSignificance(const UObject* Object) const;

	// Returns true if the object is being tracked, placing the significance value in OutSignificance (or 0 if object is not managed)
	UE_API bool QuerySignificance(const UObject* Object, float& OutSignificance) const;

	// Returns the significance manager for the specified World
	static UE_API USignificanceManager* Get(const UWorld* World);

	// Templated convenience function to return a significance manager cast to a known type
	template<class T>
	static T* Get(const UWorld* World)
	{
		return CastChecked<T>(Get(World), ECastCheckedType::NullAllowed);
	}

	// Returns the list of viewpoints currently being represented by the significance manager
	const TArray<FTransform>& GetViewpoints() const { return Viewpoints; }
	
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
protected:

	// Internal function that takes the managed object info and registers it with the significance manager
	UE_API void RegisterManagedObject(FManagedObjectInfo* ObjectInfo);

	// Callback function registered with HUD to supply debug info when ShowDebug SignificanceManager has been entered on the console
	UE_API virtual void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	// Whether the significance manager should be created on a client. Only used from CDO and 
	uint32 bCreateOnClient:1;

	// Whether the significance manager should be created on the server
	uint32 bCreateOnServer:1;

	// Whether the significance sort should sort high values to the end of the list
	uint32 bSortSignificanceAscending:1;

	// The cached viewpoints for significance for calculating when a new object is registered
	TArray<FTransform> Viewpoints;

private:

	// All objects being managed organized by Tag
	TMap<FName, TArray<FManagedObjectInfo*>> ManagedObjectsByTag;

	// Reverse lookup map to find the tag for a given object
	TMap<TObjectPtr<UObject>, FManagedObjectInfo*> ManagedObjects;

	// Array of all managed objects that we use for iteration during update. This is kept in sync with the ManagedObjects map.
	TArray<FManagedObjectInfo*> ObjArray;
	// We copy ObjArray to this before running update to avoid mutations during the update. To avoid memory allocations, making it a member.
	TArray<FManagedObjectInfo*> ObjArrayCopy;

	struct FSequentialPostWorkPair
	{
		FManagedObjectInfo* ObjectInfo;
		float OldSignificance;
	};
	// Array of all managed objects requiring sequential work that we use for iteration during update. This is kept in sync with the ManagedObjects map.
	TArray<FSequentialPostWorkPair> ObjWithSequentialPostWork;
	// We copy ObjWithSequentialPostWork to this before running update to avoid mutations during the update. To avoid memory allocations, making it a member.
	TArray<FSequentialPostWorkPair> ObjWithSequentialPostWorkCopy;

	// Game specific significance class to instantiate
	UPROPERTY(globalconfig, noclear, EditAnywhere, Category=DefaultClasses, meta=(MetaClass="/Script/SignificanceManager.SignificanceManager", DisplayName="Significance Manager Class"))
	FSoftClassPath SignificanceManagerClassName;

	// Friend FSignificanceManagerModule so it can call OnShowDebugInfo and check 
	friend class FSignificanceManagerModule;
};

#undef UE_API
